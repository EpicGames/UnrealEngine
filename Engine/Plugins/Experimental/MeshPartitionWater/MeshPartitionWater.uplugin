{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Mesh Partition Water",
	"Description": "Interoperability of Mesh Partition with the Water plugin.",
	"Category": "Mesh Partition",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "https://dev.epicgames.com/community/learning/knowledge-base/nK7J/unreal-engine-introduction-to-mesh-terrain#usingwaterbodytools",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "MeshPartitionWater",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "Water",
			"Enabled": true
		},
		{
			"Name": "MeshPartition",
			"Enabled": true
		}
	]
}