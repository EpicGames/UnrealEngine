// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MeshPartitionWater : ModuleRules
{
	public MeshPartitionWater(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		bAllowUETypesInNamespaces = true;
		
		PublicIncludePaths.AddRange(
			new string[] {
			}
			);
		
		PrivateIncludePaths.AddRange(
			new string[] {
			}
			);
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"Water",
				"MeshPartition",
				"MeshPartitionEditor",
			}
			);
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"GeometryCore",
				"GeometryAlgorithms",
				"DynamicMesh",
				"UnrealEd",
				"CQTest"
			}
			);
		
		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
			}
			);
	}
}
