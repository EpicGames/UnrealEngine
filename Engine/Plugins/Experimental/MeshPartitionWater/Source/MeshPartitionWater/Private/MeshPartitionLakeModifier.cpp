// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionLakeModifier.h"

#include "MeshPartitionEditorComponent.h"
#include "WaterBodyComponent.h"
#include "WaterSplineComponent.h"
#include "Curve/GeneralPolygon2.h"
#include "DynamicMesh/MeshNormals.h"
#include "Async/ParallelFor.h"
#include "MeshPartitionMeshView.h"

namespace UE::MeshPartition
{
namespace MegaMeshLakeModifierLocals
{
	class FBackgroundOp : public MeshPartition::IModifierBackgroundOp
	{
	public:
		FBackgroundOp(const FName& InOperationName) : MeshPartition::IModifierBackgroundOp(InOperationName) {}

		FBox GlobalBounds;
		double MaxZDistance;

		TArray<FVector3d> LakeVertices;
		double ChannelDepth;
		FVector WaterBodyComponentLocation;

		FWaterBodyHeightmapSettings WaterHeightmapSettings;
		FWaterCurveSettings CurveSettings;
		TMap<FName, FWaterBodyWeightmapSettings> WeightmapSettings;

		virtual void GetInstancesInBounds(const FBox& InBounds, TArray<FInstanceInfo>& OutInstanceInfos) const override;
		virtual void ApplyModifications(MeshPartition::FMeshView& InMeshView, const FTransform3d& InTransform,
			const FInstanceInfo& InInstanceDesc) const override;

		// Generate a new random guid before submitting any code changes to the op
		static FGuid GetCodeVersionKey()
		{
			static FGuid VersionKey(TEXT("2472f4fa-a90b-459b-97a4-65f0779dd072"));
			return VersionKey;
		}

		// Set to true whenever iterating on code changes to prevent any builds including this modifier being picked up by ddc
		// and poisoning the cache/generating lots of unused intermediate data.
		virtual bool DisableDDCWrite() const override { return false; }
	};
}


TSharedPtr<const MeshPartition::IModifierBackgroundOp> ULakeModifier::CreateBackgroundOp(const MeshPartition::EBuildType InBuildType) const
{
	using namespace MegaMeshLakeModifierLocals;

	if (!IsEnabled())
	{
		return nullptr;
	}

	const UWaterSplineComponent* SplineComp = GetWaterSpline();
	if (!ensure(SplineComp != nullptr && SplineComp->GetNumberOfSplineSegments() >= 3))
	{
		return nullptr;
	}

	const UWaterBodyComponent* WaterBodyComponent = GetWaterBodyComponent();

	TSharedPtr<MegaMeshLakeModifierLocals::FBackgroundOp> Op = MakeShared<MegaMeshLakeModifierLocals::FBackgroundOp>(GetFName());
	Op->GlobalBounds = ComputeCombinedBounds();
	Op->MaxZDistance = MaxZDistance;

	SplineComp->ConvertSplineToPolyLine(ESplineCoordinateSpace::World, FMath::Square(10.f), Op->LakeVertices);
	// Pop the last element, since it's a closed spline it should always be the same as the first
	if (ensure(Op->LakeVertices.Num() > 2 && Op->LakeVertices[0] == Op->LakeVertices.Last()))
	{
		Op->LakeVertices.Pop();
	}

	Op->ChannelDepth = WaterBodyComponent->GetChannelDepth();
	Op->WaterBodyComponentLocation = GetWaterBodyComponent()->GetComponentLocation();
	Op->WaterHeightmapSettings = WaterBodyComponent->WaterHeightmapSettings;
	Op->WeightmapSettings = WaterBodyComponent->GetLayerWeightmapSettings();
	Op->WeightmapSettings.Remove(FName()); // filter weightmaps with default/none as name
	Op->CurveSettings = WaterBodyComponent->CurveSettings;

	return Op;
}

void MegaMeshLakeModifierLocals::FBackgroundOp::GetInstancesInBounds(const FBox& InBounds, TArray<FInstanceInfo>& OutInstanceInfos) const
{
	AddDefaultInstanceIfIntersects(GlobalBounds, InBounds, OutInstanceInfos);
	if (!OutInstanceInfos.IsEmpty())
	{
		MeshPartition::ULakeModifier::RegisterWaterWeightmaps(WeightmapSettings, OutInstanceInfos.Last());
	}
}

void MegaMeshLakeModifierLocals::FBackgroundOp::ApplyModifications(MeshPartition::FMeshView& InMeshView, const FTransform3d& InTransform, const FInstanceInfo& InInstanceDesc) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::ULakeModifier::ApplyModifications);

	using namespace Geometry;


	FPolygon2d LakePolygon;
	{

		// Build a polygon describing the shape of the lake in the local space of the mega mesh.
		for (const FVector3d& Vertex : LakeVertices)
		{
			LakePolygon.AppendVertex(FVector2d(InTransform.InverseTransformPosition(Vertex)));
		}
	}
	const float LakeHeight = InTransform.InverseTransformPosition(WaterBodyComponentLocation).Z;

	ParallelFor(InMeshView.VertexCount(), [&](int VertexIndex)
	{
		FVector3d MeshVertex = InMeshView.GetVertexPos(VertexIndex);
		const bool bIsInside = LakePolygon.Contains(FVector2d(MeshVertex));
		const double Distance = FMath::Sqrt(LakePolygon.DistanceSquared(FVector2d(MeshVertex)));

		if ((MeshVertex.Z - LakeHeight) > MaxZDistance)
		{
			return;
		}

		float InternalWaterWeight = InMeshView.GetVertexAttributeWeight(MeshPartition::ULakeModifier::InternalWaterWeightChannelName, VertexIndex);

		const float NewVertexHeight = MeshPartition::ULakeModifier::CalculateVertexFalloffHeight(
			InternalWaterWeight,
			bIsInside, LakeHeight, Distance, ChannelDepth, MeshVertex.Z, WaterHeightmapSettings, CurveSettings);

		InMeshView.SetVertexAttributeWeight(MeshPartition::ULakeModifier::InternalWaterWeightChannelName, VertexIndex, InternalWaterWeight);

		for (const TPair<FName, FWaterBodyWeightmapSettings>& NameWt : WeightmapSettings)
		{
			const float ExistingWeight = InMeshView.GetVertexAttributeWeight(NameWt.Key, VertexIndex);
			// Exclusively use max blend for now. This is to alleviate problems with overlapping water bodies overwriting the weight data of others.
			const float NewWeight = FMath::Max(ExistingWeight, MeshPartition::ULakeModifier::CalculateVertexWeight(bIsInside, Distance, NameWt.Value));
			InMeshView.SetVertexAttributeWeight(NameWt.Key, VertexIndex, NewWeight);
		}

		MeshVertex.Z = NewVertexHeight;
		InMeshView.SetVertexPos(VertexIndex, MeshVertex);
	});
}

FGuid ULakeModifier::GetCodeVersionKey() const
{
	return MegaMeshLakeModifierLocals::FBackgroundOp::GetCodeVersionKey();
}
} // namespace UE::MeshPartition
