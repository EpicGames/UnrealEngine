// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MeshPartitionWaterModifier.h"
#include "MeshPartitionLakeModifier.generated.h"

#define UE_API MESHPARTITIONWATER_API

class UWaterBodyComponent;
class UWaterSplineComponent;
class AWaterBody;

namespace UE::MeshPartition
{
namespace MegaMeshLakeModifierLocals
{
	class FBackgroundOp;
}

/**
* Performs height-based MegaMesh deformation for Lakes
*/
UCLASS(MinimalAPI, meta = (BlueprintSpawnableComponent))
class ULakeModifier : public MeshPartition::UWaterModifier
{
	GENERATED_BODY()
public:
	// Begin MeshPartition::UModifierComponent Implementation
	UE_API virtual TSharedPtr<const MeshPartition::IModifierBackgroundOp> CreateBackgroundOp(const MeshPartition::EBuildType InBuildType) const override;
	UE_API virtual FGuid GetCodeVersionKey() const override;
	// End MeshPartition::UModifierComponent Implementation

	friend class MegaMeshLakeModifierLocals::FBackgroundOp; // for MeshPartition::UWaterModifier::CalculateVertexFalloffHeight
};
} // namespace UE::MeshPartition

#undef UE_API
