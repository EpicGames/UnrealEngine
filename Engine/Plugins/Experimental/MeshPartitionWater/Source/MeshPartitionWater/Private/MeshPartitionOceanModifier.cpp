// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionOceanModifier.h"

namespace UE::MeshPartition
{

TSharedPtr<const MeshPartition::IModifierBackgroundOp> UOceanModifier::CreateBackgroundOp(const MeshPartition::EBuildType InBuildType) const
{
	return nullptr;
}

} // namespace UE::MeshPartition
