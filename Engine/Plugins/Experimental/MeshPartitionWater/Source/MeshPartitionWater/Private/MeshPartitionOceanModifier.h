// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MeshPartitionWaterModifier.h"
#include "MeshPartitionOceanModifier.generated.h"

namespace UE::MeshPartition
{

UCLASS(MinimalAPI, meta = (BlueprintSpawnableComponent))
class UOceanModifier : public MeshPartition::UWaterModifier
{
	GENERATED_BODY()
public:
	// Begin MeshPartition::UModifierComponent Implementation
	virtual TSharedPtr<const MeshPartition::IModifierBackgroundOp> CreateBackgroundOp(const MeshPartition::EBuildType InBuildType) const override;
	// End MeshPartition::UModifierComponent Implementation
};

} // namespace UE::MeshPartition