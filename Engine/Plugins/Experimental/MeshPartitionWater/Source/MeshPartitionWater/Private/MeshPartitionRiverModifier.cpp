// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionRiverModifier.h"

#include "MeshPartitionEditorComponent.h"
#include "WaterBodyComponent.h"
#include "WaterSplineComponent.h"
#include "Polygon2.h"
#include "Async/ParallelFor.h"
#include "MeshPartitionMeshView.h"
#include "MeshPartitionModifierUtils.h"

namespace UE::MeshPartition
{
namespace MegaMeshRiverModifierLocals
{
	class FBackgroundOp : public MeshPartition::IModifierBackgroundOp
	{
	public:
		FBackgroundOp(const FName& InOperationName) : MeshPartition::IModifierBackgroundOp(InOperationName) {}

		FBox GlobalBounds;
		double MaxZDistance;

		TArray<FVector3d> RiverSplinePoints;
		TArray<double> Distances;

		FWaterBodyHeightmapSettings WaterHeightmapSettings;
		TMap<FName, FWaterBodyWeightmapSettings> WeightmapSettings;
		FWaterCurveSettings CurveSettings;
		FInterpCurveFloat RiverWidth;
		FInterpCurveFloat Depth;

		FTransform SplineComponentTransform;
		FVector SplineDefaultUpVector;
		FSplineCurves SplineCurves;

		virtual void GetInstancesInBounds(const FBox& InBounds, TArray<FInstanceInfo>& OutInstanceInfos) const override;
		virtual void ApplyModifications(MeshPartition::FMeshView& InMeshView, const FTransform3d& InTransform,
			const FInstanceInfo& InInstanceDesc) const override;

		// Generate a new random guid before submitting any code changes to the op
		static FGuid GetCodeVersionKey()
		{
			static FGuid VersionKey(TEXT("e766a935-fe59-4997-a2e1-0d33f983ae89"));
			return VersionKey;
		}

		// Set to true whenever iterating on code changes to prevent any builds including this modifier being picked up by ddc
		// and poisoning the cache/generating lots of unused intermediate data.
		virtual bool DisableDDCWrite() const override { return false; }
	
	private:

		FVector GetRightVectorAtDistanceAlongSpline(float Distance, ESplineCoordinateSpace::Type CoordinateSpace) const
		{
			const float Param = SplineCurves.ReparamTable.Eval(Distance, 0.0f);
			return GetRightVectorAtSplineInputKey(Param, CoordinateSpace);
		}
		FVector GetRightVectorAtSplineInputKey(float InKey, ESplineCoordinateSpace::Type CoordinateSpace) const
		{
			const FQuat Quat = GetQuaternionAtSplineInputKey(InKey, ESplineCoordinateSpace::Local);
			FVector RightVector = Quat.RotateVector(FVector::RightVector);

			if (CoordinateSpace == ESplineCoordinateSpace::World)
			{
				RightVector = SplineComponentTransform.TransformVectorNoScale(RightVector);
			}

			return RightVector;
		}
		FQuat GetQuaternionAtSplineInputKey(float InKey, ESplineCoordinateSpace::Type CoordinateSpace) const
		{
			FQuat Quat = SplineCurves.Rotation.Eval(InKey, FQuat::Identity);
			Quat.Normalize();

			const FVector Direction = SplineCurves.Position.EvalDerivative(InKey, FVector::ZeroVector).GetSafeNormal();
			const FVector UpVector = Quat.RotateVector(SplineDefaultUpVector);

			FQuat Rot = (FRotationMatrix::MakeFromXZ(Direction, UpVector)).ToQuat();

			if (CoordinateSpace == ESplineCoordinateSpace::World)
			{
				Rot = SplineComponentTransform.GetRotation() * Rot;
			}

			return Rot;
		}
		float FindInputKeyClosestToWorldLocation(const FVector& WorldLocation) const
		{
			const FVector LocalLocation = SplineComponentTransform.InverseTransformPosition(WorldLocation);
			float Dummy;
			return SplineCurves.Position.FindNearest(LocalLocation, Dummy);
		}
		FVector GetLocationAtSplineInputKey(float InKey, ESplineCoordinateSpace::Type CoordinateSpace) const
		{
			FVector Location = SplineCurves.Position.Eval(InKey, FVector::ZeroVector);

			if (CoordinateSpace == ESplineCoordinateSpace::World)
			{
				Location = SplineComponentTransform.TransformPosition(Location);
			}

			return Location;
		}
	};
}

URiverModifier::URiverModifier()
{
	MeshPartition::UModifierComponent::SetPriority(1.);
}

TArray<FBox> URiverModifier::ComputeBounds() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::URiverModifier::ComputeBounds);

	TArray<FBox> Boxes;

	if (const UWaterSplineComponent* SplineComp = GetWaterSpline())
	{
		if (const UWaterBodyComponent* WaterBodyComponent = GetWaterBodyComponent())
		{
			if (const UWaterSplineMetadata* WaterSplineMetadata = WaterBodyComponent->GetWaterSplineMetadata())
			{
				const FWaterBodyHeightmapSettings& HeightmapSettings = WaterBodyComponent->GetWaterHeightmapSettings();
				const float Falloff = HeightmapSettings.FalloffSettings.FalloffWidth;
				const float EdgeOffset = HeightmapSettings.FalloffSettings.EdgeOffset;

				auto GetWidthForDistanceAlongSpline = [&](float DistanceAlongSpline)
				{
					const float Key = SplineComp->GetInputKeyValueAtDistanceAlongSpline(DistanceAlongSpline);
					const float HalfWidth = Falloff + EdgeOffset + WaterSplineMetadata->RiverWidth.Eval(Key);
					return HalfWidth;
				};

				auto GetHeightForDistanceAlongSpline = [this](float)
				{
					return MaxZDistance;
				};

				Boxes = Utils::CollectBoundingBoxesForSpline(SplineComp, GetWidthForDistanceAlongSpline, GetHeightForDistanceAlongSpline, FMath::Square(500.));
			}
		}
	}

	return Boxes;
}


TSharedPtr<const MeshPartition::IModifierBackgroundOp> URiverModifier::CreateBackgroundOp(const MeshPartition::EBuildType InBuildType) const
{
	using namespace MegaMeshRiverModifierLocals;

	if (!IsEnabled())
	{
		return nullptr;
	}
	
	const UWaterBodyComponent* WaterBodyComponent = GetWaterBodyComponent();
	if (!ensure(WaterBodyComponent))
	{
		return nullptr;
	}

	const UWaterSplineComponent* SplineComp = GetWaterSpline();
	const UWaterSplineMetadata* WaterSplineMetadata = WaterBodyComponent->GetWaterSplineMetadata();

	if (!ensure(SplineComp != nullptr && WaterSplineMetadata != nullptr && SplineComp->GetNumberOfSplineSegments() >= 1))
	{
		return nullptr;
	}

	TSharedPtr<MegaMeshRiverModifierLocals::FBackgroundOp> Op = MakeShared<MegaMeshRiverModifierLocals::FBackgroundOp>(GetFName());
	Op->GlobalBounds = ComputeCombinedBounds();
	Op->MaxZDistance = MaxZDistance;
	Op->SplineComponentTransform = SplineComp->GetComponentTransform();
	Op->SplineDefaultUpVector = SplineComp->DefaultUpVector;
	SplineComp->ConvertSplineToPolyLineWithDistances(ESplineCoordinateSpace::Local, FMath::Square(10.f), Op->RiverSplinePoints, Op->Distances);
	Op->SplineCurves = SplineComp->GetSplineCurves();

	Op->WeightmapSettings = WaterBodyComponent->GetLayerWeightmapSettings();
	Op->WeightmapSettings.Remove(FName()); // filter weightmaps with default/none as name
	Op->WaterHeightmapSettings = WaterBodyComponent->WaterHeightmapSettings;
	Op->CurveSettings = WaterBodyComponent->CurveSettings;
	Op->RiverWidth = WaterSplineMetadata->RiverWidth;
	Op->Depth = WaterSplineMetadata->Depth;

	return Op;
}

void MegaMeshRiverModifierLocals::FBackgroundOp::GetInstancesInBounds(const FBox& InBounds, TArray<FInstanceInfo>& OutInstanceInfos) const
{
	AddDefaultInstanceIfIntersects(GlobalBounds, InBounds, OutInstanceInfos);
	if (!OutInstanceInfos.IsEmpty())
	{
		URiverModifier::RegisterWaterWeightmaps(WeightmapSettings, OutInstanceInfos.Last());
	}
}

void MegaMeshRiverModifierLocals::FBackgroundOp::ApplyModifications(MeshPartition::FMeshView& InMeshView, const FTransform3d& InTransform, const FInstanceInfo& InInstanceDesc) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::URiverModifier::ApplyModifications);

	using namespace Geometry;

	// Build a 2d polygon representation of the spline. This will be used to compute distances against for falloffs.
	FPolygon2d RiverPolygon;
	{
		const FTransform3d SplineToMeshLocal = InTransform.Inverse() * SplineComponentTransform;

		enum class EDirectionToAdd : int32
		{
			Left = 1,
			Right = -1,
		};
		
		auto AddVertexForIndex = [&](int SplinePointIndex, EDirectionToAdd DirectionToAdd)
		{
			const float DistanceAlongSpline = Distances[SplinePointIndex];
			const FVector3d Pos = RiverSplinePoints[SplinePointIndex];

			const FVector Normal = GetRightVectorAtDistanceAlongSpline(DistanceAlongSpline, ESplineCoordinateSpace::Local);

			const float Key = SplineCurves.ReparamTable.Eval(DistanceAlongSpline, 0.f);
			const float HalfWidth = RiverWidth.Eval(Key) / 2.;

			const FVector3d Vertex  = SplineToMeshLocal.TransformPosition(Pos + Normal * HalfWidth * static_cast<int32>(DirectionToAdd));
			RiverPolygon.AppendVertex(FVector2d(Vertex));
		};

		// Traverse the river forwards adding only the left vertices then traverse the river backwards adding only the right vertices
		// This creates a clockwise wound polygon describing the river shape.
		for (int SplinePointIndex = 0; SplinePointIndex < Distances.Num(); ++SplinePointIndex)
		{
			AddVertexForIndex(SplinePointIndex, EDirectionToAdd::Left);
		}
		for (int SplinePointIndex = Distances.Num() - 1; SplinePointIndex >= 0; --SplinePointIndex)
		{
			AddVertexForIndex(SplinePointIndex, EDirectionToAdd::Right);
		}
	}


	ParallelFor(InMeshView.VertexCount(), [&](int VertexIndex)
	{
		FVector3d MeshVertex = InMeshView.GetVertexPos(VertexIndex);
		const bool bIsInside = RiverPolygon.Contains(FVector2d(MeshVertex));
		const double Distance = FMath::Sqrt(RiverPolygon.DistanceSquared(FVector2d(MeshVertex)));

		const float SplineInputKey = FindInputKeyClosestToWorldLocation(InTransform.TransformPosition(MeshVertex));
		const float RiverHeight = InTransform.InverseTransformPosition(GetLocationAtSplineInputKey(SplineInputKey, ESplineCoordinateSpace::World)).Z;
		const float TargetDepth = Depth.Eval(SplineInputKey);

		if ((MeshVertex.Z - RiverHeight) > MaxZDistance)
		{
			return;
		}

		float InternalWaterWeight = InMeshView.GetVertexAttributeWeight(MeshPartition::URiverModifier::InternalWaterWeightChannelName, VertexIndex);

		const float NewVertexHeight = URiverModifier::CalculateVertexFalloffHeight(
			InternalWaterWeight, 
			bIsInside, RiverHeight, Distance, TargetDepth, MeshVertex.Z, WaterHeightmapSettings, CurveSettings);

		InMeshView.SetVertexAttributeWeight(MeshPartition::URiverModifier::InternalWaterWeightChannelName, VertexIndex, InternalWaterWeight);

		for (const TPair<FName, FWaterBodyWeightmapSettings>& NameWt : WeightmapSettings)
		{
			const float ExistingWeight = InMeshView.GetVertexAttributeWeight(NameWt.Key, VertexIndex);
			// Exclusively use max blend for now. This is to alleviate problems with overlapping water bodies overwriting the weight data of others.
			const float NewWeight = FMath::Max(ExistingWeight, MeshPartition::URiverModifier::CalculateVertexWeight(bIsInside, Distance, NameWt.Value));
			InMeshView.SetVertexAttributeWeight(NameWt.Key, VertexIndex, NewWeight);
		}

		MeshVertex.Z = NewVertexHeight;
		InMeshView.SetVertexPos(VertexIndex, MeshVertex);
	});
}

FGuid URiverModifier::GetCodeVersionKey() const
{
	return MegaMeshRiverModifierLocals::FBackgroundOp::GetCodeVersionKey();
}
} // namespace UE::MeshPartition