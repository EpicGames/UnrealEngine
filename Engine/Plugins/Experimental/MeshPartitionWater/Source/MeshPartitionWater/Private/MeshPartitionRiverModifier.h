// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MeshPartitionWaterModifier.h"

#include "MeshPartitionRiverModifier.generated.h"

#define UE_API MESHPARTITIONWATER_API

namespace UE::MeshPartition
{
namespace MegaMeshRiverModifierLocals
{
	class FBackgroundOp;
}

/**
* Performs height-based MegaMesh deformation for Rivers
*/
UCLASS(MinimalAPI, meta = (BlueprintSpawnableComponent))
class URiverModifier : public MeshPartition::UWaterModifier
{
	GENERATED_BODY()
public:
	UE_API URiverModifier();
	
	// Begin MeshPartition::UModifierComponent Implementation
	UE_API virtual TArray<FBox> ComputeBounds() const override;
	UE_API virtual TSharedPtr<const MeshPartition::IModifierBackgroundOp> CreateBackgroundOp(const MeshPartition::EBuildType InBuildType) const override;
	UE_API virtual FGuid GetCodeVersionKey() const override;
	// End MeshPartition::UModifierComponent Implementation

	friend class MegaMeshRiverModifierLocals::FBackgroundOp; // for MeshPartition::UWaterModifier::CalculateVertexFalloffHeight
};
}

#undef UE_API
