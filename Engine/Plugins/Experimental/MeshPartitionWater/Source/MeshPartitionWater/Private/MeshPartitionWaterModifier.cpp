// Copyright Epic Games, Inc. All Rights Reserved.


#include "MeshPartitionWaterModifier.h"

#include "MeshPartition.h"
#include "MeshPartitionEditorComponent.h"
#include "MeshPartitionModule.h" // FCustomVersion
#include "WaterBodyActor.h"
#include "WaterBodyComponent.h"
#include "WaterTerrainComponent.h"
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"

#define LOCTEXT_NAMESPACE "MegaMeshWaterModifier"

namespace UE::MeshPartition
{

FName UWaterModifier::InternalWaterWeightChannelName("Water_InternalBlendChannel");

UWaterModifier::UWaterModifier()
{
}

void UWaterModifier::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	const FName ParentPreviousDefaultType = TEXT("Misc");
	if (Ar.IsLoading()
		&& GetLinkerCustomVersion(UE::MeshPartition::FCustomVersion::GUID) < UE::MeshPartition::FCustomVersion::DefaultPriorityLayerSetToNone
		&& GetType() == ParentPreviousDefaultType) // Super::Serialize would set default to this
	{
		const FName PreviousModifierDefaultType = TEXT("WaterBody");
		SetType(PreviousModifierDefaultType);
	}
}

void UWaterModifier::OnRegister()
{
	Super::OnRegister();

	if (AWaterBody* WaterBodyActor = GetWaterBodyActor())
	{
		WaterBodyActor->GetOnWaterBrushActorChangedEvent().AddUObject(this, &MeshPartition::UWaterModifier::OnWaterBodyChanged);
	}
}

void UWaterModifier::OnUnregister()
{
	if (AWaterBody* WaterBodyActor = GetWaterBodyActor())
	{
		WaterBodyActor->GetOnWaterBrushActorChangedEvent().RemoveAll(this);
	}
	Super::OnUnregister();
}


TArray<FBox> UWaterModifier::ComputeBounds() const
{
	FBox Box(ForceInitToZero);

	if (const UWaterBodyComponent* WaterBodyComponent = GetWaterBodyComponent())
	{
		float FalloffDistance = 0.f;
		const FWaterBodyHeightmapSettings& HeightmapSettings = WaterBodyComponent->GetWaterHeightmapSettings();
		if (HeightmapSettings.FalloffSettings.FalloffMode == EWaterBrushFalloffMode::Width)
		{
			FalloffDistance = HeightmapSettings.FalloffSettings.FalloffWidth + HeightmapSettings.FalloffSettings.EdgeOffset;
		}
		for (const TPair<FName, FWaterBodyWeightmapSettings>& WeightmapSetting : WaterBodyComponent->GetLayerWeightmapSettings())
		{
			FalloffDistance = FMath::Max(FalloffDistance, WeightmapSetting.Value.FalloffWidth + WeightmapSetting.Value.EdgeOffset);
		}
		FBox AffectedBounds = WaterBodyComponent->CalcBounds(WaterBodyComponent->GetComponentTransform()).GetBox().ExpandBy(FalloffDistance);
		Box = AffectedBounds.ExpandBy(FVector(0.f, 0.f, MaxZDistance));
	}
	return { Box }; 
}

void UWaterModifier::PostProcessSection(AActor* InSection)
{
	if (InSection->GetComponentByClass(UWaterTerrainComponent::StaticClass()) == nullptr)
	{
		UWaterTerrainComponent* TerrainComponent = NewObject<UWaterTerrainComponent>(InSection, UWaterTerrainComponent::StaticClass(), TEXT("WaterTerrainComponent"));

		InSection->AddInstanceComponent(TerrainComponent);
		TerrainComponent->OnComponentCreated();
		TerrainComponent->RegisterComponent();
	}
}

bool UWaterModifier::IsEnabled() const
{
	return GetWaterBodyComponent() ? GetWaterBodyComponent()->AffectsLandscape() : false;
}

void UWaterModifier::CheckForErrors()
{
	const UWaterBodyComponent* WaterBodyComponent = GetWaterBodyComponent();
	if (WaterBodyComponent == nullptr)
	{
		FMessageLog("MapCheck").Warning()
			->AddToken(FUObjectToken::Create(this))
			->AddToken(FTextToken::Create(LOCTEXT("MapCheck_MegaMeshWaterModifier_MissingComponent", "Mega Mesh Water Modifier could not find the affecting Water Body Component!")));
	}

	Super::CheckForErrors();
}

UWaterBodyComponent* UWaterModifier::GetWaterBodyComponent() const
{
	if (AWaterBody* WaterBodyActor = GetWaterBodyActor())
	{
		return WaterBodyActor->GetWaterBodyComponent();
	}
	return nullptr;
}

AWaterBody* UWaterModifier::GetWaterBodyActor() const
{
	return GetOwner<AWaterBody>();
}

UWaterSplineComponent* UWaterModifier::GetWaterSpline() const
{
	if (const UWaterBodyComponent* WaterBodyComponent = GetWaterBodyComponent())
	{
		return WaterBodyComponent->GetWaterSpline();
	}
	return nullptr;
}

void UWaterModifier::OnWaterBodyChanged(const IWaterBrushActorInterface::FWaterBrushActorChangedEventParams& OnBrushActorChanged)
{
	if (!IsEnabled())
	{
		return;
	}
	if (OnBrushActorChanged.WaterBrushActor != GetWaterBodyActor())
	{
		return;
	}
	
	if (OnBrushActorChanged.PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		OnChanged(ComputeBounds(), UE::MeshPartition::EChangeType::StateChange);
	}
}

float UWaterModifier::CalculateVertexFalloffHeight(float& InOutInternalBlendWeight, bool bIsInside, float WaterHeight, float DistanceFromSpline, float TargetDepth, float MeshZ, const FWaterBodyHeightmapSettings& HeightmapSettings, const FWaterCurveSettings& CurveSettings)
{
	float TargetHeight = 0;
	float TargetInternalWeight = 0;
	float SourceInternalWeight = InOutInternalBlendWeight;
	
	// Find the target water height and weight depending on whether we're inside or outside of the water shape
	if (bIsInside)
	{
		const double Alpha = FMath::Clamp((DistanceFromSpline - CurveSettings.ChannelEdgeOffset) / CurveSettings.CurveRampWidth, 0., 1.0);

		TargetHeight = FMath::Lerp(WaterHeight, WaterHeight - TargetDepth, Alpha);
		// Inside the water, the weight is above 1
		TargetInternalWeight = 1.f + Alpha;
	}
	else
	{
		float FalloffDistance = 0.f;
		if (HeightmapSettings.FalloffSettings.FalloffMode == EWaterBrushFalloffMode::Angle)
		{
			FalloffDistance = FMath::Abs((WaterHeight - MeshZ) / FMath::Tan(FMath::DegreesToRadians(HeightmapSettings.FalloffSettings.FalloffAngle)));
		}
		else
		{
			FalloffDistance = HeightmapSettings.FalloffSettings.FalloffWidth;
		}

		const double Alpha = FMath::Clamp((DistanceFromSpline - HeightmapSettings.FalloffSettings.EdgeOffset) / FalloffDistance, 0., 1.0);

		TargetHeight = WaterHeight;
		// Outside the water, the weight falls off to 0
		TargetInternalWeight = 1.f-Alpha;
	}
	
	// start at the min of terrain and water target height
	float ResultHeight = FMath::Min(TargetHeight, MeshZ);

	// if we're in the outer-blend of old water, blend toward the new water's target height
	if (SourceInternalWeight < 1.f)
	{
		ResultHeight = FMath::Lerp(ResultHeight, TargetHeight, 1.f - SourceInternalWeight);
	}
	// if we're in the outer-blend of new water, blend toward previous landscape height
	if (TargetInternalWeight < 1.f)
	{
		ResultHeight = FMath::Lerp(ResultHeight, MeshZ, 1.f - TargetInternalWeight);
	}

	InOutInternalBlendWeight = FMath::Max(SourceInternalWeight, TargetInternalWeight);
	return ResultHeight;
}

float UWaterModifier::CalculateVertexWeight(bool bIsInside, float DistanceFromSpline, const FWaterBodyWeightmapSettings& WeightmapSettings)
{
	// Note: Not currently using WeightmapSettings's FinalOpacity, Midpoint, ModulationTexture, etc
	return (bIsInside ? 1.f : 1.f - FMath::Clamp((DistanceFromSpline - WeightmapSettings.EdgeOffset) / WeightmapSettings.FalloffWidth, 0.f, 1.f));
}

void UWaterModifier::RegisterWaterWeightmaps(const TMap<FName, FWaterBodyWeightmapSettings>& WeightMaps, FInstanceInfo& Instance)
{
	Instance.UsedChannels.Emplace(InternalWaterWeightChannelName);
	Instance.ReadViewComponents |= EMeshViewComponents::VertexAttributeWeight;
	Instance.WriteViewComponents |= EMeshViewComponents::VertexAttributeWeight;

	for (const TPair<FName, FWaterBodyWeightmapSettings>& WeightChannelEntry : WeightMaps)
	{
		if (!WeightChannelEntry.Key.IsNone())
		{
			Instance.UsedChannels.Emplace(WeightChannelEntry.Key);
		}
	}
}

} // namespace UE::MeshPartition

#undef LOCTEXT_NAMESPACE
