// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS
#if WITH_EDITOR

#include "Test/MeshPartitionTestUtils.h"
#include "EditorWorldUtils.h"
#include "MeshPartition.h"
#include "MeshPartitionEditorComponent.h"
#include "MeshPartitionModifierComponent.h"
#include "MeshPartitionPreviewSection.h"
#include "Engine/StaticMesh.h"
#include "WaterBodyLakeActor.h"
#include "MeshPartitionLakeModifier.h"
#include "WaterBodyRiverActor.h"
#include "MeshPartitionRiverModifier.h"
#include "Editor.h"
#include "ActorFactories/ActorFactory.h"
#include "CQTest.h"
#include "Tests/AutomationCommon.h"
#include "Tests/AutomationEditorCommon.h"

using namespace UE::MeshPartition;

/**
 * Tests for MeshPartition Water Modifier
 */
TEST_CLASS_WITH_FLAGS(Water, "MeshPartition", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
{
	// Test environment and core objects
	TUniquePtr<FScopedEditorWorld> ScopedEditorWorld;
	// CVars
	TOptional<TestUtils::FScopedCVarOverride> DisableDDCWrite;
	TOptional<TestUtils::FScopedCVarOverride> DisableDDCRead;

	// MegaMesh components under test
	UWorld* World = nullptr;
	AMeshPartition* MegaMesh = nullptr;
	UMeshPartitionEditorComponent* MeshPartitionEditorComponent = nullptr;
	URiverModifier* RiverModifier = nullptr;
	ULakeModifier* LakeModifier = nullptr;

	// Ground truth mesh data for validation
	TArray<TObjectPtr<UStaticMesh>> GroundTruth;

	const FString LevelPath = TEXT("/MeshPartition/UnitTests/MeshPartitionUnitTests");
	const FString ReferencePath = TEXT("/MeshPartitionWater/UnitTests");

	/*
	 * Defines the types of water body configurations to test
	 */
	enum class EWaterTestType
	{
		Lake,
		River,
		LakeAndRiver
	};

	/*
	 * Configuration parameters for water body creation
	 */
	struct FWaterBodyConfig
	{
		UE::Math::TRotator<double> Rotation = { 0., 0., 0. };
		UE::Math::TVector<double> Translation = { 0., 0., 0. };
		float FalloffWidth = 2048.f;
		EWaterBrushFalloffMode FalloffMode = EWaterBrushFalloffMode::Width;
	};

	/*
	 * Returns ground truth mesh asset names for each water test type.
	 */
	static const TMap<EWaterTestType, const FString>& GetGroundTruthAsset()
	{
		static TMap<EWaterTestType, const FString> GroundTruthPaths;

		if (GroundTruthPaths.Num() == 0)
		{
			GroundTruthPaths.Add(
				EWaterTestType::Lake,
				TEXT("Lake_PreviewSection")
			);
			GroundTruthPaths.Add(
				EWaterTestType::River,
				TEXT("River_PreviewSection")
			);
			GroundTruthPaths.Add(
				EWaterTestType::LakeAndRiver,
				TEXT("CombinedLakeAndRiverModifierGroundTruth")
			);
		}

		return GroundTruthPaths;
	}

	BEFORE_EACH()
	{
		// Configure minimal editor world
		FWorldInitializationValues InitializationValues;
		InitializationValues.RequiresHitProxies(false);
		InitializationValues.ShouldSimulatePhysics(false);
		InitializationValues.EnableTraceCollision(false);
		InitializationValues.CreateNavigation(false);
		InitializationValues.CreateAISystem(false);
		InitializationValues.AllowAudioPlayback(false);
		InitializationValues.CreatePhysicsScene(true);

#if AUTOMATED_TESTING_EXECUTE_IN_EDITOR_WORLD
		World = GEditor->GetEditorWorldContext().World();
#else
		// Spawn world
		ScopedEditorWorld = FAutomationEditorCommonUtils::CreateScopedEditorWorld(LevelPath, InitializationValues);
		ASSERT_THAT(IsNotNull(ScopedEditorWorld.Get(), TEXT("Failed to create scoped editor world")));

		World = ScopedEditorWorld->GetWorld();
#endif
		ASSERT_THAT(IsNotNull(World, TEXT("World is not valid")));

		// Need to disable DDC read/write, so that generated preview sections aren't written to DDC
		DisableDDCWrite.Emplace(TEXT("MegaMesh.Preview.EnableDDCWrite"), TEXT("0"));
		DisableDDCRead.Emplace(TEXT("MegaMesh.Preview.EnableDDCRead"), TEXT("0"));
	}

	AFTER_EACH()
	{
		ScopedEditorWorld.Reset();
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
	}

	TEST_METHOD(Verify_LakeModifier_Generates_Correct_Mesh)
	{
		RunWaterModifierTestForWaterType(EWaterTestType::Lake);
	}

	TEST_METHOD(Verify_RiverModifier_Generates_Correct_Mesh)
	{
		RunWaterModifierTestForWaterType(EWaterTestType::River);
	}

	TEST_METHOD(Verify_CombinedLakeAndRiverModifier_Generates_Correct_Mesh)
	{
		RunWaterModifierTestForWaterType(EWaterTestType::LakeAndRiver);
	}

	TEST_METHOD(Verify_RiverModifier_MaxZDistance_Updates_Mesh)
	{
		TestCommandBuilder
			.Do(TEXT("Create MeshTerrain"), [this]()
				{
					// Create sectioned MegaMesh for testing
					MegaMesh = TestUtils::CreateTestMeshSectioned(World, 2, 2, 10000, 10000, 50, 50);
					ASSERT_THAT(IsNotNull(MegaMesh, TEXT("Failed to create MegaMesh")));

					MeshPartitionEditorComponent = Cast<UMeshPartitionEditorComponent>(MegaMesh->GetMeshPartitionComponent());
				})
			.Then(TEXT("Create river modifier"), [this]()
				{
					const FWaterBodyConfig DefaultRiverConfig{
						{0., 0., 0. },
						{ -500, 4500, 5000. },
						2048.,
						EWaterBrushFalloffMode::Width };

					RiverModifier = CreateWaterBodyWithModifier<AWaterBodyRiver, URiverModifier>(
						EWaterTestType::River, DefaultRiverConfig
					);

					MeshPartitionEditorComponent->OnModifierAssigned();
				})
			.Until(TEXT("Wait for preview section build"), [this]()
				{
					return !MeshPartitionEditorComponent->IsAnyPreviewSectionBuildActive();
				})
			.Then(TEXT("Validate initial mesh"), [this]()
				{
					APreviewSection* PreviewSection = nullptr;
					int32 PreviewSectionCount = 0;

					MeshPartitionEditorComponent->ForAllPreviewSections([&](APreviewSection* InPreviewSection)
						{
							PreviewSection = InPreviewSection;
							++PreviewSectionCount;
							return true;
						});
					ASSERT_THAT(AreEqual(PreviewSectionCount, 1, TEXT("PreviewSection count should be 1")));
					ASSERT_THAT(IsNotNull(PreviewSection, TEXT("PreviewSection should not be null")));

					const FString GroundTruthAsset = TEXT("RiverModifierDefaultZHeightGroundTruth");
					CompareToReferenceAsset(GroundTruthAsset, MeshPartitionEditorComponent);

					// Update MaxZDistance parameter
					RiverModifier->SetMaxZDistance(1.f);
					RiverModifier->OnChanged({
						FBox(FVector3d(-HALF_WORLD_MAX, -HALF_WORLD_MAX, -HALF_WORLD_MAX),
						FVector3d(HALF_WORLD_MAX, HALF_WORLD_MAX, HALF_WORLD_MAX)) },
						EChangeType::StateChange);
					MeshPartitionEditorComponent->Update();
				})
			.Until(TEXT("Wait for preview section build"), [this]()
				{
					return !MeshPartitionEditorComponent->IsAnyPreviewSectionBuildActive();
				})
			.Then(TEXT("Validate updated mesh with MaxZDistance"), [this]()
				{
					APreviewSection* PreviewSection = nullptr;
					int32 PreviewSectionCount = 0;
					MeshPartitionEditorComponent->ForAllPreviewSections([&](APreviewSection* InPreviewSection)
						{
							PreviewSection = InPreviewSection;
							++PreviewSectionCount;
							return true;
						});
					ASSERT_THAT(AreEqual(PreviewSectionCount, 1, TEXT("PreviewSectionCount should be 1")));
					ASSERT_THAT(IsNotNull(PreviewSection, TEXT("PreviewSection should not be null")));

					const FString GroundTruthAsset = TEXT("RiverModifierModifiedZHeightGroundTruth");
					CompareToReferenceAsset(GroundTruthAsset, MeshPartitionEditorComponent);
				});
	}

	private:
	/**
	 * Executes a water modifier test for the specified water type configuration.
	 * Creates the appropriate water bodies, waits for preview generation and validates against ground truth.
	 * 
	 * @param Type	The Type of water configuration to test (Lake, River, or both)
	 */
	void RunWaterModifierTestForWaterType(EWaterTestType Type)
	{
		TestCommandBuilder
			.Do(TEXT("Setup test environment"), [this, Type]()
				{
					MegaMesh = TestUtils::CreateTestMesh(World, 20000, 20000, 100, 100);
					ASSERT_THAT(IsNotNull(MegaMesh, TEXT("Failed to create test MegaMesh")));

					MeshPartitionEditorComponent = Cast<UMeshPartitionEditorComponent>(MegaMesh->GetMeshPartitionComponent());
				})
			.Then(TEXT("Create water body modifiers"), [this, Type]()
				{
					// Default configurations for each water body type
					const FWaterBodyConfig DefaultLakeConfig {
						{0., 0., 0. },
						{ -4000, -2000, 1500. },
						2048.,
						EWaterBrushFalloffMode::Width };

					const FWaterBodyConfig DefaultRiverConfig{
						{0., 0., 0. },
						{ -5000, -2000, 1500. },
						2048.,
						EWaterBrushFalloffMode::Width };

					switch (Type)
					{
					case EWaterTestType::Lake:
						LakeModifier = CreateWaterBodyWithModifier<AWaterBodyLake, ULakeModifier>(
							EWaterTestType::Lake, DefaultLakeConfig
						);
						break;
					case EWaterTestType::River:
						RiverModifier = CreateWaterBodyWithModifier<AWaterBodyRiver, URiverModifier>(
							EWaterTestType::River, DefaultRiverConfig
						);
						break;
					case EWaterTestType::LakeAndRiver:
						LakeModifier = CreateWaterBodyWithModifier<AWaterBodyLake, ULakeModifier>(
							EWaterTestType::Lake, DefaultLakeConfig
						);
						RiverModifier = CreateWaterBodyWithModifier<AWaterBodyRiver, URiverModifier>(
							EWaterTestType::River, DefaultRiverConfig
						);
						break;

					default: ASSERT_FAIL(TEXT("WaterTestType is not set"));
					}

					// Notify MegaMesh that modifiers have been assigned
					MeshPartitionEditorComponent->OnModifierAssigned();
				})
			.Until(TEXT("Wait for preview section generation"), [this]()
				{
					return !MeshPartitionEditorComponent->IsAnyPreviewSectionBuildActive();
				})
			.Then(TEXT("Validate preview section and mesh"), [this, Type]()
				{
					APreviewSection* PreviewSection = nullptr;
					int32 PreviewSectionCount = 0;
					MeshPartitionEditorComponent->ForAllPreviewSections([&](APreviewSection* InPreviewSection)
						{
							PreviewSection = InPreviewSection;
							++PreviewSectionCount;
							return true;
						});

					ASSERT_THAT(AreEqual(PreviewSectionCount, 1, TEXT("Expected PreviewSectionCount to be 1")));
					ASSERT_THAT(IsNotNull(PreviewSection, TEXT("PreviewSection can not be null")));

					const auto& GroundTruthAsset = GetGroundTruthAsset()[Type];
					CompareToReferenceAsset(GroundTruthAsset, MeshPartitionEditorComponent);
				});
	}

	/**
	 * Creates a water body actor with an attached MegaMesh modifier component.
	 * 
	 * @tparam TWaterBodyActor		Water Body Actor class
	 * @tparam TModifierComponent	ModifierComponent for Water Body Actor class
	 * @param Type					The type of water test being performed
	 * @param Config				Configuration parameters for the water body
	 * @return						The created modifier component
	 */
	template<typename TWaterBodyActor, typename TModifierComponent>
	TModifierComponent* CreateWaterBodyWithModifier(
		const EWaterTestType Type,
		const FWaterBodyConfig& Config
	)
	{
		UActorFactory* Factory = GEditor->FindActorFactoryForActorClass(TWaterBodyActor::StaticClass());
		TWaterBodyActor* WaterBody = Cast<TWaterBodyActor>(
			Factory->CreateActor(
				TWaterBodyActor::StaticClass(),
				World->GetCurrentLevel(),
				FTransform(Config.Rotation, Config.Translation)
			)
		);

		// Configure water body falloff settings
		UWaterBodyComponent* WaterBodyComponent = WaterBody->GetWaterBodyComponent();
		WaterBodyComponent->WaterHeightmapSettings.FalloffSettings.FalloffMode = Config.FalloffMode;
		WaterBodyComponent->WaterHeightmapSettings.FalloffSettings.FalloffWidth = Config.FalloffWidth;

		// Create and configure MegaMesh modifier component
		TModifierComponent* Modifier = NewObject<TModifierComponent>(
			WaterBody,
			TModifierComponent::StaticClass(),
			Type == EWaterTestType::Lake ? TEXT("LakeModifier") : TEXT("RiverModifier")
		);

		// Link modifier to MegaMesh and register
		Modifier->SetAffectedMeshPartition(MegaMesh);
		WaterBody->AddInstanceComponent(Modifier);
		Modifier->AttachToComponent(WaterBodyComponent, FAttachmentTransformRules::KeepWorldTransform);
		Modifier->RegisterComponent();

		return Modifier;
	}

	void CompareToReferenceAsset(const FString& TestName, const UMeshPartitionEditorComponent* MeshPartitionComponent)
	{
		const FString ReferencePackagePath = ReferencePath / TestName;
		const FString ReferenceAssetPath = ReferencePackagePath + "." + TestName;

		if (TestUtils::ShouldUpdateReferenceAssets())
		{
			[[maybe_unused]] const bool bSuccess =
				TestUtils::CreateOrUpdateReference(MeshPartitionComponent, ReferencePackagePath, { TestName }, TestName);
		}

		FString ErrorMessage;
		bool bValid = TestUtils::ValidatePreviewMeshAgainstGroundTruth(
			MeshPartitionComponent,
			{ ReferenceAssetPath },
			ErrorMessage);

		ASSERT_THAT(IsTrue(bValid, *ErrorMessage));
	}
};

#endif // WITH_EDITOR
#endif // WITH_DEV_AUTOMATION_TESTS