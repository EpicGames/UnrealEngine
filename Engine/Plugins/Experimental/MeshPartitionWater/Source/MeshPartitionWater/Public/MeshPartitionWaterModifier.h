// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MeshPartitionModifierComponent.h"
#include "WaterBrushActorInterface.h"
#include "MeshPartitionWaterModifier.generated.h"

#define UE_API MESHPARTITIONWATER_API

class UWaterBodyComponent;
class UWaterSplineComponent;
class AWaterBody;

namespace UE::MeshPartition
{
/**
* Base MegaMeshModifierComponent for all Water-MegaMesh interactions.
*/
UCLASS(MinimalAPI, Abstract, meta=(BlueprintSpawnableComponent))
class UWaterModifier : public MeshPartition::UModifierComponent
{
	GENERATED_BODY()

public:
	UE_API UWaterModifier();

	// UObject
	UE_API virtual void Serialize(FArchive& Ar) override;

	// USceneComponent Implementation
	UE_API virtual void OnRegister() override;
	UE_API virtual void OnUnregister() override;
	UE_API virtual void CheckForErrors() override;
	// End USceneComponent Implementation

	// Begin MeshPartition::UModifierComponent Implementation
	UE_API virtual TArray<FBox> ComputeBounds() const override;
	UE_API virtual void PostProcessSection(AActor* InSection) override;
	// End MeshPartition::UModifierComponent Implementation

	UE_API bool IsEnabled() const;

	void SetMaxZDistance(const double InMaxZDistance) { MaxZDistance = InMaxZDistance; }
	
protected:

	UE_API void OnWaterBodyChanged(const IWaterBrushActorInterface::FWaterBrushActorChangedEventParams& OnBrushActorChanged);

	static UE_API float CalculateVertexFalloffHeight(float& InOutInternalBlendWeight, bool bIsInside, float WaterHeight, float DistanceFromSpline, float TargetHeight, float MeshZ, const FWaterBodyHeightmapSettings& HeightmapSettings, const FWaterCurveSettings& CurveSettings);
	// helper to calculate basic weightmap alpha (not accounting for ModulationTexture)
	static UE_API float CalculateVertexWeight(bool bIsInside, float DistanceFromSpline, const FWaterBodyWeightmapSettings& WeightmapSettings);
	// helper to register water weightmaps
	static UE_API void RegisterWaterWeightmaps(const TMap<FName, FWaterBodyWeightmapSettings>& WeightMaps, FInstanceInfo& Instance);

	/** Utilities functions to retrieve water components from the parent water body actor */
	UE_API AWaterBody* GetWaterBodyActor() const;
	UE_API UWaterBodyComponent* GetWaterBodyComponent() const;
	UE_API UWaterSplineComponent* GetWaterSpline() const;

protected:
	/* The maximum vertical distance this modifier will affect the terrain */
	UPROPERTY(EditAnywhere, Category = Terrain)
	double MaxZDistance = 10000.;

	/* Name of the weight channel used to pass height blend info between overlapping water modifiers */
	static UE_API FName InternalWaterWeightChannelName;
};
} // namespace UE::MeshPartition

#undef UE_API
